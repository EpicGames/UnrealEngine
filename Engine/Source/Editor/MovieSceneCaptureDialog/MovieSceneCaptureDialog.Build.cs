// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class MovieSceneCaptureDialog : ModuleRules
{
	public MovieSceneCaptureDialog(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"InputCore",
				"Json",
				"JsonUtilities",
				"MovieScene",
				"MovieSceneCapture",
				"MovieSceneTools",
				"PropertyEditor",
				"SessionServices",
				"Slate",
				"SlateCore",
				"EditorFramework",
				"UnrealEd",
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
			}
		);
	}
}
