// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneCaptureDialogModule.h"
#include "Dom/JsonValue.h"
#include "Dom/JsonObject.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "Layout/Margin.h"
#include "GenericPlatform/GenericApplication.h"
#include "Input/Reply.h"
#include "UObject/GCObject.h"
#include "Widgets/SWidget.h"
#include "Misc/Paths.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/CoreStyle.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/Docking/TabManager.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "FrameNumberDetailsCustomization.h"

#include "HAL/IConsoleManager.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/FileManager.h"
#include "Misc/CommandLine.h"
#include "Misc/StringBuilder.h"
#include "Misc/FileHelper.h"
#include "Misc/App.h"
#include "Serialization/ObjectWriter.h"
#include "Serialization/ObjectReader.h"
#include "Templates/SubclassOf.h"
#include "Engine/GameViewportClient.h"
#include "GameFramework/GameModeBase.h"
#include "Slate/SceneViewport.h"
#include "MovieSceneCapture.h"

#include "Serialization/JsonSerializer.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Input/SButton.h"
#include "Settings/LevelEditorPlaySettings.h"
#include "Editor.h"

#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Images/SThrobber.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "AudioDevice.h"

#include "Widgets/Docking/SDockTab.h"
#include "JsonObjectConverter.h"
#include "Widgets/Notifications/INotificationWidget.h"

#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "FileHelpers.h"

#include "ISessionManager.h"
#include "ISessionServicesModule.h"

#include "ErrorCodes.h"

#include "GameFramework/WorldSettings.h"
#include "FrameNumberNumericInterface.h"
#include "Protocols/AudioCaptureProtocol.h"

#define LOCTEXT_NAMESPACE "MovieSceneCaptureDialog"

const TCHAR* MovieCaptureSessionName = TEXT("Movie Scene Capture");

DECLARE_DELEGATE_RetVal_OneParam(FText, FOnStartCapture, UMovieSceneCapture*);

class SRenderMovieSceneSettings : public SCompoundWidget, public FGCObject
{
	SLATE_BEGIN_ARGS(SRenderMovieSceneSettings) : _InitialObject(nullptr) {}
		SLATE_EVENT(FOnStartCapture, OnStartCapture)
		SLATE_ARGUMENT(UMovieSceneCapture*, InitialObject)
		SLATE_ARGUMENT(TSharedPtr<INumericTypeInterface<double>>, NumericTypeInterface)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		FPropertyEditorModule& PropertyEditor = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.ViewIdentifier = "RenderMovieScene";

		DetailView = PropertyEditor.CreateDetailView(DetailsViewArgs);
		DetailView->RegisterInstancedCustomPropertyTypeLayout("FrameNumber", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FFrameNumberDetailsCustomization::MakeInstance, InArgs._NumericTypeInterface));

		OnStartCapture = InArgs._OnStartCapture;

		ChildSlot
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			[
				DetailView.ToSharedRef()
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(ErrorText, STextBlock)
				.Visibility(EVisibility::Hidden)
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(5.f)
			[
				SNew(SButton)
				.IsEnabled(this, &SRenderMovieSceneSettings::CanStartCapture)
				.ContentPadding(FMargin(10, 5))
				.Text(this, &SRenderMovieSceneSettings::GetStartCaptureText)
				.OnClicked(this, &SRenderMovieSceneSettings::OnStartClicked)
			]
			
		];

		MovieSceneCapture = nullptr;

		if (InArgs._InitialObject)
		{
			SetObject(InArgs._InitialObject);
		}
	}

	void SetObject(UMovieSceneCapture* InMovieSceneCapture)
	{
		MovieSceneCapture = InMovieSceneCapture;

		DetailView->SetObject(InMovieSceneCapture);

		ErrorText->SetText(FText());
		ErrorText->SetVisibility(EVisibility::Hidden);
	}

	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override
	{
		Collector.AddReferencedObject(MovieSceneCapture);
	}
	virtual FString GetReferencerName() const override
	{
		return TEXT("SRenderMovieSceneSettings");
	}

private:

	FReply OnStartClicked()
	{
		FText Error;
		if (OnStartCapture.IsBound())
		{
			Error = OnStartCapture.Execute(MovieSceneCapture);
		}

		ErrorText->SetText(Error);
		ErrorText->SetVisibility(Error.IsEmpty() ? EVisibility::Hidden : EVisibility::Visible);

		return FReply::Handled();
	}

	FText GetStartCaptureText() const
	{
		if (MovieSceneCapture && !MovieSceneCapture->bUseSeparateProcess)
		{
			for (const FWorldContext& Context : GEngine->GetWorldContexts())
			{
				if (Context.WorldType == EWorldType::PIE)
				{
					return LOCTEXT("ExportExitPIE", "(Exit PIE to start)");
				}
			}
		}

		return LOCTEXT("Export", "Capture Movie");
	}

	bool CanStartCapture() const
	{
		if (!MovieSceneCapture)
		{
			return false;
		}
		else if (MovieSceneCapture->bUseSeparateProcess)
		{
			return true;
		}

		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			if (Context.WorldType == EWorldType::PIE)
			{
				return false;
			}
		}

		return true;
	}

	TSharedPtr<IDetailsView> DetailView;
	TSharedPtr<STextBlock> ErrorText;
	FOnStartCapture OnStartCapture;
	TObjectPtr<UMovieSceneCapture> MovieSceneCapture;
};

DECLARE_DELEGATE_OneParam(FOnCaptureFinished, bool /*bCancelled*/);

class SCaptureMovieNotification : public SCompoundWidget, public INotificationWidget
{
public:
	SLATE_BEGIN_ARGS(SCaptureMovieNotification){}

		SLATE_ATTRIBUTE(FCaptureState, CaptureState)

		SLATE_EVENT(FOnCaptureFinished, OnCaptureFinished)

		SLATE_EVENT(FSimpleDelegate, OnCancel)

		SLATE_ARGUMENT(FString, CapturePath)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		CaptureState = InArgs._CaptureState;
		OnCaptureFinished = InArgs._OnCaptureFinished;
		OnCancel = InArgs._OnCancel;

		CachedState = FCaptureState(ECaptureStatus::Pending);

		FString CapturePath = FPaths::ConvertRelativePathToFull(InArgs._CapturePath);
		CapturePath.RemoveFromEnd(TEXT("\\"));

		auto OnBrowseToFolder = [=]{
			FString TrimmedPath;
			if (CapturePath.Split(TEXT("{"), &TrimmedPath, nullptr))
			{
				FPaths::NormalizeDirectoryName(TrimmedPath);
				FPlatformProcess::ExploreFolder(*TrimmedPath);
			}
			else
			{
				FPlatformProcess::ExploreFolder(*CapturePath);
			}
		};

		ChildSlot
		[
			SNew(SBox)
			.Padding(FMargin(15.0f))
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.Padding(FMargin(0,0,0,5.0f))
				.HAlign(HAlign_Right)
				.AutoHeight()
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					[
						SAssignNew(TextBlock, STextBlock)
						.Font(FCoreStyle::Get().GetFontStyle(TEXT("NotificationList.FontBold")))
						.Text(LOCTEXT("RenderingVideo", "Capturing video"))
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(15.f,0,0,0))
					[
						SAssignNew(Throbber, SThrobber)
					]
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(FMargin(0,0,0,5.0f))
				.HAlign(HAlign_Right)
				[
					SAssignNew(DetailedTextBlock, STextBlock)
					.Visibility(EVisibility::Collapsed)
					.Font(FCoreStyle::Get().GetFontStyle(TEXT("NotificationList.FontLight")))
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Right)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SAssignNew(Hyperlink, SHyperlink)
						.Text(LOCTEXT("OpenFolder", "Open Capture Folder..."))
						.OnNavigate_Lambda(OnBrowseToFolder)
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(5.0f,0,0,0))
					.VAlign(VAlign_Center)
					[
						SAssignNew(Button, SButton)
						.Text(LOCTEXT("StopButton", "Stop Capture"))
						.OnClicked(this, &SCaptureMovieNotification::ButtonClicked)
					]
				]
			]
		];
	}

	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
	{
		if (State != SNotificationItem::CS_Pending)
		{
			return;
		}

		FCaptureState StateThisFrame = CaptureState.Get();

		if (CachedState.Status != StateThisFrame.Status)
		{
			CachedState = StateThisFrame;
			
			if (CachedState.Status == ECaptureStatus::Success)
			{
				TextBlock->SetText(LOCTEXT("CaptureFinished", "Capture Finished"));
				OnCaptureFinished.ExecuteIfBound(true);
			}
			else if (CachedState.Status == ECaptureStatus::Failure)
			{
				TextBlock->SetText(LOCTEXT("CaptureFailed", "Capture Failed"));
				FText DetailText = CachedState.GetDetailText();
				if (!DetailText.IsEmpty())
				{
					DetailedTextBlock->SetText(DetailText);
					DetailedTextBlock->SetVisibility(EVisibility::Visible);
				}
				OnCaptureFinished.ExecuteIfBound(false);
			}
			else
			{
				ensureMsgf(false, TEXT("Cannot move from a finished to a pending state."));
			}
		}
	}

	virtual void OnSetCompletionState(SNotificationItem::ECompletionState InState)
	{
		State = InState;
		if (State != SNotificationItem::CS_Pending)
		{
			Throbber->SetVisibility(EVisibility::Collapsed);
			Button->SetVisibility(EVisibility::Collapsed);
		}
	}

	virtual TSharedRef< SWidget > AsWidget()
	{
		return AsShared();
	}

private:

	FReply ButtonClicked()
	{
		if (State == SNotificationItem::CS_Pending)
		{
			OnCancel.ExecuteIfBound();
		}
		return FReply::Handled();
	}
	
private:
	TSharedPtr<SWidget> Button, Throbber, Hyperlink;
	TSharedPtr<STextBlock> TextBlock;
	TSharedPtr<STextBlock> DetailedTextBlock;
	SNotificationItem::ECompletionState State;

	FSimpleDelegate OnCancel;
	FCaptureState CachedState;
	TAttribute<FCaptureState> CaptureState;
	FOnCaptureFinished OnCaptureFinished;
};

void FInEditorCapture::Start()
{
	ULevelEditorPlaySettings* PlayInEditorSettings = GetMutableDefault<ULevelEditorPlaySettings>();

	bScreenMessagesWereEnabled = GAreScreenMessagesEnabled;
	GAreScreenMessagesEnabled = false;

	if (!CaptureObject->Settings.bEnableTextureStreaming)
	{
		const int32 UndefinedTexturePoolSize = -1;
		IConsoleVariable* CVarStreamingPoolSize = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Streaming.PoolSize"));
		if (CVarStreamingPoolSize)
		{
			BackedUpStreamingPoolSize = CVarStreamingPoolSize->GetInt();
			CVarStreamingPoolSize->Set(UndefinedTexturePoolSize, ECVF_SetByConsole);
		}

		IConsoleVariable* CVarUseFixedPoolSize = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Streaming.UseFixedPoolSize"));
		if (CVarUseFixedPoolSize)
		{
			BackedUpUseFixedPoolSize = CVarUseFixedPoolSize->GetInt(); 
			CVarUseFixedPoolSize->Set(0, ECVF_SetByConsole);
		}

		IConsoleVariable* CVarTextureStreaming = IConsoleManager::Get().FindConsoleVariable(TEXT("r.TextureStreaming"));
		if (CVarTextureStreaming)
		{
			BackedUpTextureStreaming = CVarTextureStreaming->GetInt();
			CVarTextureStreaming->Set(0, ECVF_SetByConsole);
		}
	}

	FObjectWriter(PlayInEditorSettings, BackedUpPlaySettings);
	OverridePlaySettings(PlayInEditorSettings);

	CaptureObject->AddToRoot();
	CaptureObject->OnCaptureFinished().AddRaw(this, &FInEditorCapture::OnLevelSequenceFinished);

	UGameViewportClient::OnViewportCreated().AddRaw(this, &FInEditorCapture::OnPIEViewportStarted);
	FEditorDelegates::EndPIE.AddRaw(this, &FInEditorCapture::OnEndPIE);
		
	FAudioDeviceHandle AudioDevice = GEngine->GetMainAudioDevice();
	if (AudioDevice)
	{
		TransientPrimaryVolume = AudioDevice->GetTransientPrimaryVolume();
		AudioDevice->SetTransientPrimaryVolume(0.0f);
	}

	TSharedRef<SWindow> CustomWindow = SNew(SWindow)
		.Title(LOCTEXT("MovieRenderPreviewTitle", "Movie Render - Preview"))
		.AutoCenter(EAutoCenter::PrimaryWorkArea)
		.UseOSWindowBorder(true)
		.FocusWhenFirstShown(false)
		.ActivationPolicy(EWindowActivationPolicy::Never)
		.HasCloseButton(true)
		.SupportsMaximize(false)
		.SupportsMinimize(true)
		.MaxWidth(CaptureObject->GetSettings().Resolution.ResX)
		.MaxHeight(CaptureObject->GetSettings().Resolution.ResY)
		.SizingRule(ESizingRule::FixedSize);

	FSlateApplication::Get().AddWindow(CustomWindow);

	FRequestPlaySessionParams Params;
	Params.EditorPlaySettings = PlayInEditorSettings;
	Params.CustomPIEWindow = CustomWindow;

	GEditor->RequestPlaySession(Params);
}

void FInEditorCapture::Cancel()
{
	// If the user cancels through the UI then we request that the editor shut down the PIE instance.
	// We capture the PIE shutdown request (which calls OnEndPIE) and further process it. This unifies
	// closing PIE via the close button and the UI into one code path.
	GEditor->RequestEndPlayMap();
}

void FInEditorCapture::OverridePlaySettings(ULevelEditorPlaySettings* PlayInEditorSettings)
{
	const FMovieSceneCaptureSettings& Settings = CaptureObject->GetSettings();

	PlayInEditorSettings->NewWindowWidth = Settings.Resolution.ResX;
	PlayInEditorSettings->NewWindowHeight = Settings.Resolution.ResY;
	PlayInEditorSettings->CenterNewWindow = false;
	PlayInEditorSettings->NewWindowPosition = FIntPoint::NoneValue; // It will center PIE to the middle of the screen the first time it is run (until the user drag the window somewhere else)
	PlayInEditorSettings->LastExecutedPlayModeType = EPlayModeType::PlayMode_InEditorFloating;

	// Reset everything else
	PlayInEditorSettings->GameGetsMouseControl = false;
	PlayInEditorSettings->ShowMouseControlLabel = false;
	PlayInEditorSettings->ViewportGetsHMDControl = false;
	PlayInEditorSettings->ShouldMinimizeEditorOnVRPIE = true;
	PlayInEditorSettings->EnableGameSound = CaptureObject->AudioCaptureProtocolType != UNullAudioCaptureProtocol::StaticClass();
	PlayInEditorSettings->bOnlyLoadVisibleLevelsInPIE = false;
	PlayInEditorSettings->bPreferToStreamLevelsInPIE = false;
	PlayInEditorSettings->PIEAlwaysOnTop = false;
	PlayInEditorSettings->DisableStandaloneSound = false;
	PlayInEditorSettings->AdditionalLaunchParameters = TEXT("");
	PlayInEditorSettings->BuildGameBeforeLaunch = EPlayOnBuildMode::PlayOnBuild_Never;
	PlayInEditorSettings->LaunchConfiguration = EPlayOnLaunchConfiguration::LaunchConfig_Default;
	PlayInEditorSettings->PackFilesForLaunch = EPlayOnPakFileMode::NoPak;
	PlayInEditorSettings->SetPlayNetMode(EPlayNetMode::PIE_Standalone);
	PlayInEditorSettings->SetRunUnderOneProcess(true);
	PlayInEditorSettings->bLaunchSeparateServer = false;
	PlayInEditorSettings->SetPlayNumberOfClients(1);
}

void FInEditorCapture::OnPIEViewportStarted()
{
	for (const FWorldContext& Context : GEngine->GetWorldContexts())
	{
		if (Context.WorldType == EWorldType::PIE)
		{
			FSlatePlayInEditorInfo* SlatePlayInEditorSession = GEditor->SlatePlayInEditorMap.Find(Context.ContextHandle);
			if (SlatePlayInEditorSession)
			{
				CapturingFromWorld = Context.World();

				TSharedPtr<SWindow> Window = SlatePlayInEditorSession->SlatePlayInEditorWindow.Pin();

				const FMovieSceneCaptureSettings& Settings = CaptureObject->GetSettings();

				SlatePlayInEditorSession->SlatePlayInEditorWindowViewport->SetViewportSize(Settings.Resolution.ResX,Settings.Resolution.ResY);

				FVector2D PreviewWindowSize(Settings.Resolution.ResX, Settings.Resolution.ResY);

				// Keep scaling down the window size while we're bigger than half the desktop width/height
				{
					FDisplayMetrics DisplayMetrics;
					FSlateApplication::Get().GetCachedDisplayMetrics(DisplayMetrics);
						
					while(PreviewWindowSize.X >= DisplayMetrics.PrimaryDisplayWidth*.5f || PreviewWindowSize.Y >= DisplayMetrics.PrimaryDisplayHeight*.5f)
					{
						PreviewWindowSize *= .5f;
					}
				}
					
				// Resize and move the window into the desktop a bit
				FVector2D PreviewWindowPosition(50, 50);
				Window->ReshapeWindow(PreviewWindowPosition, PreviewWindowSize);

				if (CaptureObject->Settings.GameModeOverride != nullptr)
				{
					CachedGameMode = CapturingFromWorld->GetWorldSettings()->DefaultGameMode;
					CapturingFromWorld->GetWorldSettings()->DefaultGameMode = CaptureObject->Settings.GameModeOverride;
				}

				CachedEngineShowFlags = SlatePlayInEditorSession->SlatePlayInEditorWindowViewport->GetClient()->GetEngineShowFlags();
				if (CachedEngineShowFlags && Settings.bUsePathTracer)
				{
					CachedPathTracingMode = CachedEngineShowFlags->PathTracing;
					CachedEngineShowFlags->SetPathTracing(true);
				}
				CaptureObject->Initialize(SlatePlayInEditorSession->SlatePlayInEditorWindowViewport, Context.PIEInstance);
				OnCaptureStarted();
			}
			return;
		}
	}

	UE_LOGF(LogTemp, Warning, "Received PIE Creation callback but failed to find PIE World or missing FSlatePlayInEditorInfo for world.");
}

void FInEditorCapture::Shutdown()
{
	FEditorDelegates::EndPIE.RemoveAll(this);
	UGameViewportClient::OnViewportCreated().RemoveAll(this);
	CaptureObject->OnCaptureFinished().RemoveAll(this);

	GAreScreenMessagesEnabled = bScreenMessagesWereEnabled;

	if (!CaptureObject->Settings.bEnableTextureStreaming)
	{
		IConsoleVariable* CVarStreamingPoolSize = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Streaming.PoolSize"));
		if (CVarStreamingPoolSize)
		{
			CVarStreamingPoolSize->Set(BackedUpStreamingPoolSize, ECVF_SetByConsole);
		}

		IConsoleVariable* CVarUseFixedPoolSize = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Streaming.UseFixedPoolSize"));
		if (CVarUseFixedPoolSize)
		{
			CVarUseFixedPoolSize->Set(BackedUpUseFixedPoolSize, ECVF_SetByConsole);
		}

		IConsoleVariable* CVarTextureStreaming = IConsoleManager::Get().FindConsoleVariable(TEXT("r.TextureStreaming"));
		if (CVarTextureStreaming)
		{
			CVarTextureStreaming->Set(BackedUpTextureStreaming, ECVF_SetByConsole);
		}
	}

	if (CaptureObject->Settings.GameModeOverride != nullptr && CapturingFromWorld != nullptr)
	{
		CapturingFromWorld->GetWorldSettings()->DefaultGameMode = CachedGameMode;
	}
	
	if (CachedEngineShowFlags)
	{
		CachedEngineShowFlags->SetPathTracing(CachedPathTracingMode);
	}

	FObjectReader(GetMutableDefault<ULevelEditorPlaySettings>(), BackedUpPlaySettings);

	FAudioDeviceHandle AudioDevice = GEngine->GetMainAudioDevice();
	if (AudioDevice)
	{
		AudioDevice->SetTransientPrimaryVolume(TransientPrimaryVolume);
	}

	CaptureObject->Close();
	CaptureObject->RemoveFromRoot();
	
}

void FInEditorCapture::OnEndPIE(bool bIsSimulating)
{
	Shutdown();
}

void FInEditorCapture::OnLevelSequenceFinished()
{
	Shutdown();

	GEditor->RequestEndPlayMap();
}

void FInEditorCapture::OnCaptureStarted()
{
	FString CapturePath = CaptureObject->ResolveFileFormat(CaptureObject->Settings.OutputDirectory.Path, FFrameMetrics());

	FNotificationInfo Info
	(
		SNew(SCaptureMovieNotification)
		.CaptureState_Raw(this, &FInEditorCapture::GetCaptureState)
		.CapturePath(CapturePath)
		.OnCaptureFinished_Raw(this, &FInEditorCapture::OnCaptureFinished)
		.OnCancel_Raw(this, &FInEditorCapture::Cancel)
	);

	Info.bFireAndForget = false;
	Info.ExpireDuration = 5.f;
	InProgressCaptureNotification = FSlateNotificationManager::Get().AddNotification(Info);
	InProgressCaptureNotification->SetCompletionState(SNotificationItem::CS_Pending);
}

FCaptureState FInEditorCapture::GetCaptureState() const
{
	for (const FWorldContext& Context : GEngine->GetWorldContexts())
	{
		if (Context.WorldType == EWorldType::PIE)
		{
			return FCaptureState(ECaptureStatus::Pending);
		}
	}

	return FCaptureState(ECaptureStatus::Success);
}

void FMovieSceneCaptureBase::OnCaptureFinished(bool bSuccess)
{
	if (bSuccess)
	{
		InProgressCaptureNotification->SetCompletionState(SNotificationItem::CS_Success);
	}
	else
	{
		UE_LOGF(LogTemp, Error, "MovieSceneCapture failed to capture.");
		InProgressCaptureNotification->SetCompletionState(SNotificationItem::CS_Fail);
	}

	InProgressCaptureNotification->ExpireAndFadeout();
	InProgressCaptureNotification = nullptr;

	if (OnFinishedCallback)
	{
		OnFinishedCallback(bSuccess);
	}
}

void FNewProcessCapture::Start()
{
	// Save out the capture manifest to json
	FString Filename = FPaths::ProjectSavedDir() / TEXT("MovieSceneCapture/Manifest.json");

	TSharedRef<FJsonObject> Object = MakeShareable(new FJsonObject);
	if (FJsonObjectConverter::UStructToJsonObject(CaptureObject->GetClass(), CaptureObject, Object, 0, 0))
	{
		TSharedRef<FJsonObject> RootObject = MakeShareable(new FJsonObject);
		RootObject->SetField(TEXT("Type"), MakeShareable(new FJsonValueString(CaptureObject->GetClass()->GetPathName())));
		RootObject->SetField(TEXT("Data"), MakeShareable(new FJsonValueObject(Object)));

		TSharedRef<FJsonObject> AdditionalJson = MakeShareable(new FJsonObject);
		CaptureObject->SerializeJson(*AdditionalJson);
		RootObject->SetField(TEXT("AdditionalData"), MakeShareable(new FJsonValueObject(AdditionalJson)));

		FString Json;
		TSharedRef<TJsonWriter<> > JsonWriter = TJsonWriterFactory<>::Create(&Json, 0);
		if (FJsonSerializer::Serialize(RootObject, JsonWriter))
		{
			FFileHelper::SaveStringToFile(Json, *Filename);
		}
	}
	else
	{
		return;
	}

	FString EditorCommandLine = FString::Printf(TEXT("%s -MovieSceneCaptureManifest=\"%s\" -game -NoLoadingScreen -ForceRes -Windowed"), *MapNameToLoad, *Filename);

	// Spit out any additional, user-supplied command line args
	if (!CaptureObject->AdditionalCommandLineArguments.IsEmpty())
	{
		EditorCommandLine.AppendChar(' ');
		EditorCommandLine.Append(CaptureObject->AdditionalCommandLineArguments);
	}

	// Spit out any inherited command line args
	if (!CaptureObject->InheritedCommandLineArguments.IsEmpty())
	{
		EditorCommandLine.AppendChar(' ');
		EditorCommandLine.Append(CaptureObject->InheritedCommandLineArguments);
	}

	// Disable texture streaming if necessary
	if (!CaptureObject->Settings.bEnableTextureStreaming)
	{
		EditorCommandLine.Append(TEXT(" -NoTextureStreaming"));
	}

	// Set the game resolution - we always want it windowed
	EditorCommandLine += FString::Printf(TEXT(" -ResX=%d -ResY=%d -Windowed"), CaptureObject->Settings.Resolution.ResX, CaptureObject->Settings.Resolution.ResY);

	// Ensure game session is correctly set up 
	EditorCommandLine += FString::Printf(TEXT(" -messaging -SessionName=\"%s\""), MovieCaptureSessionName);

	TStringBuilder<64> SubprocessCommandLine;
	FCommandLine::BuildSubprocessCommandLine(ECommandLineArgumentFlags::GameContexts, false /*bOnlyInherited*/, SubprocessCommandLine);
	FString Params;
	if (FPaths::IsProjectFilePathSet())
	{
		Params = FString::Printf(TEXT("\"%s\" %s %s"), *FPaths::GetProjectFilePath(), *EditorCommandLine, *SubprocessCommandLine);
	}
	else
	{
		Params = FString::Printf(TEXT("%s %s %s"), FApp::GetProjectName(), *EditorCommandLine, *SubprocessCommandLine);
	}

	FString GamePath = FPlatformProcess::GenerateApplicationPath(FApp::GetName(), FApp::GetBuildConfiguration());
	FProcHandle ProcessHandle = FPlatformProcess::CreateProc(*GamePath, *Params, true, false, false, nullptr, 0, nullptr, nullptr);

	if (ProcessHandle.IsValid())
	{
		if (CaptureObject->bCloseEditorWhenCaptureStarts)
		{
			FPlatformMisc::RequestExit(false);
			return;
		}

		SharedProcHandle = MakeShareable(new FProcHandle(ProcessHandle));

		OnCaptureStarted();
	}
	else
	{
		OnCaptureFinished(false);
	}
}

void FNewProcessCapture::Cancel()
{
	// If they cancel the capture via the UI we need to try and find a running session with the right name
	bool bFoundInstance = false;

	// Attempt to send a remote command to gracefully terminate the process
	ISessionServicesModule& SessionServices = FModuleManager::Get().LoadModuleChecked<ISessionServicesModule>("SessionServices");
	TSharedPtr<ISessionManager> SessionManager = SessionServices.GetSessionManager();

	TArray<TSharedPtr<ISessionInfo>> Sessions;
	if (SessionManager.IsValid())
	{
		SessionManager->GetSessions(Sessions);
	}

	for (const TSharedPtr<ISessionInfo>& Session : Sessions)
	{
		if (Session->GetSessionName() == MovieCaptureSessionName)
		{
			TArray<TSharedPtr<ISessionInstanceInfo>> Instances;
			Session->GetInstances(Instances);

			for (const TSharedPtr<ISessionInstanceInfo>& Instance : Instances)
			{
				Instance->ExecuteCommand("exit");
				bFoundInstance = true;
			}
		}
	}

	if (!bFoundInstance)
	{
		FPlatformProcess::TerminateProc(*SharedProcHandle);
	}
}

void FNewProcessCapture::OnCaptureStarted()
{
	FNotificationInfo Info
	(
		SNew(SCaptureMovieNotification)
		.CaptureState_Raw(this, &FNewProcessCapture::GetCaptureState)
		.CapturePath(CaptureObject->Settings.OutputDirectory.Path)
		.OnCaptureFinished_Raw(this, &FNewProcessCapture::OnCaptureFinished)
		.OnCancel_Raw(this, &FNewProcessCapture::Cancel)
	);

	Info.bFireAndForget = false;
	Info.ExpireDuration = 5.f;
	InProgressCaptureNotification = FSlateNotificationManager::Get().AddNotification(Info);
	InProgressCaptureNotification->SetCompletionState(SNotificationItem::CS_Pending);
}

FCaptureState FNewProcessCapture::GetCaptureState() const
{
	if (!FPlatformProcess::IsProcRunning(*SharedProcHandle))
	{
		int32 RetCode = 0;
		FPlatformProcess::GetProcReturnCode(*SharedProcHandle, &RetCode);
		return FCaptureState(RetCode);
	}
	else
	{
		return FCaptureState(ECaptureStatus::Pending);
	}
}

class FMovieSceneCaptureDialogModule : public IMovieSceneCaptureDialogModule
{
	virtual UWorld* GetCurrentlyRecordingWorld() override
	{
		return CurrentCapture.IsValid() ? CurrentCapture->GetWorld() : nullptr;
	}

	virtual TSharedPtr<FMovieSceneCaptureBase> GetCurrentCapture() const override
	{
		return CurrentCapture;
	}

	virtual void StartCapture(UMovieSceneCapture* InCaptureSettings) override
	{
		/** Called when the capture object finishes its capture (success or fail). */
		auto OnCaptureFinished = [this](bool bSuccess)
		{
			UE_LOGF(LogTemp, Log, "Movie Capture finished. Success: %d", bSuccess);
			
			// CurrentCapture has to be null so that StartRecording can be called again which means we can't
			// broadcast this until after we've nulled out Current Capture. Thus we store the delegate
			FMovieSceneCaptureBase::FCaptureStateStopped Delegate = CurrentCapture->CaptureStoppedDelegate;
			
			CurrentCapture = nullptr;
			Delegate.Broadcast(bSuccess);
		};

		if (InCaptureSettings->bUseSeparateProcess)
		{
			const FString WorldPackageName = GWorld->GetOutermost()->GetName();
			FString MapNameToLoad = WorldPackageName;

			// Allow the game mode to be overridden
			if (InCaptureSettings->Settings.GameModeOverride != nullptr)
			{
				const FString GameModeName = InCaptureSettings->Settings.GameModeOverride->GetPathName();
				MapNameToLoad += FString::Printf(TEXT("?game=%s"), *GameModeName);
			}

			CurrentCapture = MakeShared<FNewProcessCapture>(InCaptureSettings, MapNameToLoad, OnCaptureFinished);
		}
		else
		{
			CurrentCapture = MakeShared<FInEditorCapture>(InCaptureSettings, OnCaptureFinished);
		}

		UE_LOGF(LogTemp, Log, "Starting movie scene capture...");
		CurrentCapture->Start();
	}

	virtual void OpenDialog(const TSharedRef<FTabManager>& TabManager, UMovieSceneCapture* CaptureObject, TSharedPtr<INumericTypeInterface<double>> InNumericTypeInterface) override
	{
		// Ensure the session services module is loaded otherwise we won't necessarily receive status updates from the movie capture session
		FModuleManager::Get().LoadModuleChecked<ISessionServicesModule>("SessionServices").GetSessionManager();

		TSharedPtr<SWindow> ExistingWindow = CaptureSettingsWindow.Pin();
		if (ExistingWindow.IsValid())
		{
			ExistingWindow->BringToFront();
		}
		else
		{
			ExistingWindow = SNew(SWindow)
				.Title( LOCTEXT("RenderMovieSettingsTitle", "Render Movie Settings") )
				.HasCloseButton(true)
				.SupportsMaximize(false)
				.SupportsMinimize(false)
				.ClientSize(FVector2D(500, 700));

			TSharedPtr<SDockTab> OwnerTab = TabManager->GetOwnerTab();
			TSharedPtr<SWindow> RootWindow = OwnerTab.IsValid() ? OwnerTab->GetParentWindow() : TSharedPtr<SWindow>();
			if(RootWindow.IsValid())
			{
				FSlateApplication::Get().AddWindowAsNativeChild(ExistingWindow.ToSharedRef(), RootWindow.ToSharedRef());
			}
			else
			{
				FSlateApplication::Get().AddWindow(ExistingWindow.ToSharedRef());
			}
		}

		ExistingWindow->SetContent(
			SNew(SRenderMovieSceneSettings)
			.InitialObject(CaptureObject)
			.NumericTypeInterface(InNumericTypeInterface)
			.OnStartCapture_Raw(this, &FMovieSceneCaptureDialogModule::OnUserRequestStartCapture)
		);

		CaptureSettingsWindow = ExistingWindow;
	}

	FText OnUserRequestStartCapture(UMovieSceneCapture* CaptureObject)
	{
		if (CurrentCapture.IsValid())
		{
			return LOCTEXT("AlreadyCapturing", "There is already a movie scene capture process open. Please close it and try again.");
		}

		FString OutputDirectory = CaptureObject->Settings.OutputDirectory.Path;
		FPaths::NormalizeFilename(OutputDirectory);

		// Only validate the directory if it doesn't contain any format specifiers
		if (!OutputDirectory.Contains(TEXT("{")))
		{
			if (!IFileManager::Get().DirectoryExists(*OutputDirectory))
			{
				if (!IFileManager::Get().MakeDirectory(*OutputDirectory))
				{
					return FText::Format(LOCTEXT( "InvalidDirectory", "Invalid output directory: {0}"), FText::FromString(OutputDirectory) );
				}
			}
			else if (IFileManager::Get().IsReadOnly(*OutputDirectory))
			{
				return FText::Format(LOCTEXT( "ReadOnlyDirectory", "Read only output directory: {0}"), FText::FromString(OutputDirectory) );
			}
		}

		// Prompt the user to save their changes so that they'll be in the movie, since we're not saving temporary copies of the level.
		bool bPromptUserToSave = true;
		bool bSaveMapPackages = true;
		bool bSaveContentPackages = true;
		if( !FEditorFileUtils::SaveDirtyPackages( bPromptUserToSave, bSaveMapPackages, bSaveContentPackages ) )
		{
			return LOCTEXT( "UserCancelled", "Capturing was cancelled from the save dialog." );
		}

		CaptureObject->SaveToConfig();

		StartCapture(CaptureObject);

		// If we managed to get this far, we've done our best to start the capture and don't have a error to report at this time.
		return FText();
	}
private:
	TWeakPtr<SWindow> CaptureSettingsWindow;
	TSharedPtr<FMovieSceneCaptureBase> CurrentCapture;
};

FText FCaptureState::GetDetailText() const 
{
	switch (uint32(Code))
	{
	case uint32(EMovieSceneCaptureExitCode::WorldNotFound):
	{
		return LOCTEXT("WorldNotFound", "Specified world does not exist. Did you forget to save it?");
	}
	default:
	{
		return FText();
	}
	}
}

IMPLEMENT_MODULE( FMovieSceneCaptureDialogModule, MovieSceneCaptureDialog )

#undef LOCTEXT_NAMESPACE
