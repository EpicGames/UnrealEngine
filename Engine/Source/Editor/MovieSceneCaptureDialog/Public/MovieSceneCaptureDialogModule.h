// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "GameFramework/GameModeBase.h"
#include "Widgets/Input/NumericTypeInterface.h"

#define UE_API MOVIESCENECAPTUREDIALOG_API

// Forward Declare
class UMovieSceneCapture;
class SNotificationItem;
struct FMovieSceneCaptureBase;

enum class ECaptureStatus
{
	Pending,
	Success,
	Failure,
	Unknown
};

class IMovieSceneCaptureDialogModule : public IModuleInterface
{
public:
	static IMovieSceneCaptureDialogModule& Get()
	{
		static const FName ModuleName(TEXT("MovieSceneCaptureDialog"));
		return FModuleManager::LoadModuleChecked<IMovieSceneCaptureDialogModule>(ModuleName);
	}
	virtual void OpenDialog(const TSharedRef<class FTabManager>& TabManager, UMovieSceneCapture* CaptureObject, TSharedPtr<INumericTypeInterface<double>> InNumericTypeInterface) = 0;

	/** Get the world we're currently recording from, if an in process record is happening */
	virtual UWorld* GetCurrentlyRecordingWorld() = 0;

	virtual TSharedPtr<FMovieSceneCaptureBase> GetCurrentCapture() const = 0;
	virtual void StartCapture(UMovieSceneCapture* InCaptureSettings) = 0;
};

/** Stores the Capture State for display in the UI. Combines both capture status and any additional context information. */
struct FCaptureState
{
	/** Construction from an enum */
	explicit FCaptureState(ECaptureStatus InState = ECaptureStatus::Unknown) 
		: Status(InState)
		, Code(0) {}
	/** Construction from a process exit code */
	explicit FCaptureState(int32 InCode)
		: Status(InCode == 0 ? ECaptureStatus::Success : ECaptureStatus::Failure)
		, Code(InCode) {}

	/** Get any additional detailed text */
	FText GetDetailText() const;

	ECaptureStatus Status;
	int32 Code;
};

struct FMovieSceneCaptureBase : TSharedFromThis<FMovieSceneCaptureBase>
{
public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FCaptureStateStopped, bool);

	virtual ~FMovieSceneCaptureBase() = default;
	virtual void OnCaptureStarted() {}
	UE_API virtual void OnCaptureFinished(bool bSuccess);
	virtual UWorld* GetWorld() const { return nullptr; }
	virtual void Start() {}
	virtual void Cancel() {}
	virtual FCaptureState GetCaptureState() const
	{
		return FCaptureState();
	}

public:
	/** Multicast Delegate for when capture is stopped. Returns true if the capture was completed successfully. */
	FCaptureStateStopped CaptureStoppedDelegate;

protected:
	/** Pointer to the capture notification pop up. */
	TSharedPtr<SNotificationItem> InProgressCaptureNotification;

	/** Cached copy of our capture object. */
	UMovieSceneCapture* CaptureObject;

	/** Callback to call when we finish capturing. */
	TFunction<void(bool)> OnFinishedCallback;
};

struct FInEditorCapture : FMovieSceneCaptureBase
{
	FInEditorCapture(UMovieSceneCapture* InCaptureObject, const TFunction<void(bool)>& InOnFinishedCallback)
	{
		CapturingFromWorld = nullptr;
		CaptureObject = InCaptureObject;
		OnFinishedCallback = InOnFinishedCallback;
	}

	// FMovieSceneCaptureBase interface
	virtual UWorld* GetWorld() const override 
	{
		// Return a reference to the PIE world.
		return CapturingFromWorld;
	}
	UE_API virtual void OnCaptureStarted() override;
	UE_API virtual void Start() override;
	UE_API virtual void Cancel() override;
	UE_API virtual FCaptureState GetCaptureState() const override;
	// ~FMovieSceneCaptureBase interface

private:
	/** Overrides the Level Editor Play settings to specifically disable some things (such as audio playback) */
	UE_API void OverridePlaySettings(class ULevelEditorPlaySettings* PlayInEditorSettings);
	/** Called when the PIE Viewport is created. */
	UE_API void OnPIEViewportStarted();
	/** Shuts down the capture setup , called when PIE is closed by the user or the sequence finishes playing. */
	UE_API void Shutdown();
	/** Called when the user closes the PIE instance window. */
	UE_API void OnEndPIE(bool bIsSimulating);
	/** Called when the Sequence finishes playing to the end. */
	UE_API void OnLevelSequenceFinished();

private:
	UWorld* CapturingFromWorld;

	bool bScreenMessagesWereEnabled;
	float TransientPrimaryVolume;
	int32 BackedUpStreamingPoolSize;
	int32 BackedUpUseFixedPoolSize;
	int32 BackedUpTextureStreaming;
	TArray<uint8> BackedUpPlaySettings;

	bool CachedPathTracingMode = false;
	struct FEngineShowFlags* CachedEngineShowFlags = nullptr;
	TSubclassOf<AGameModeBase> CachedGameMode;
};

struct FNewProcessCapture : FMovieSceneCaptureBase
{

	FNewProcessCapture(UMovieSceneCapture* InCaptureObject, const FString& InMapNameToLoad, const TFunction<void(bool)>& InOnFinishedCallback)
	{
		CaptureObject = InCaptureObject;
		MapNameToLoad = InMapNameToLoad;
		OnFinishedCallback = InOnFinishedCallback;

		SharedProcHandle = nullptr;
	}

	// FMovieSceneCaptureBase
	UE_API virtual void Start() override;
	UE_API virtual void Cancel() override;
	UE_API virtual void OnCaptureStarted() override;
	UE_API virtual FCaptureState GetCaptureState() const override;
	// ~FMovieSceneCaptureBase

protected:
	TSharedPtr<FProcHandle> SharedProcHandle;
	FString MapNameToLoad;
};

#undef UE_API
